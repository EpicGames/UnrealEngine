// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MacPlatformEditor : ModuleRules
{
	public MacPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"DesktopPlatform",
				"Engine",
				"MainFrame",
				"Slate",
				"SlateCore",
				"PropertyEditor",
				"SharedSettingsWidgets",
				"SourceControl",
				"MacTargetPlatformSettings",
				"MacTargetPlatformControls",
				"TargetPlatform",
				"MaterialShaderQualitySettings",
				"RenderCore",
				"SettingsEditor",
				"AudioSettingsEditor",
				"RHI",
				"UnrealEd",
				"ShaderPlatformConfigEditor",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
				"Settings",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"GameProjectGeneration",
				}
		);
	}
}
