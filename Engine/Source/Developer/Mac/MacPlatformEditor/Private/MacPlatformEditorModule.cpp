// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "MacTargetSettings.h"
#include "XcodeProjectSettings.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Class.h"
#include "PropertyEditorModule.h"
#include "XcodeProjectSettingsDetailsCustomization.h"
#include "MacTargetSettingsCustomization.h"

#define LOCTEXT_NAMESPACE "MacPlatformEditorModule"

/**
 * Module for Mac project settings
 */
class FMacPlatformEditorModule
	: public IModuleInterface
{
	// IModuleInterface interface

	virtual void StartupModule() override
	{
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "Mac",
				LOCTEXT("MacTargetSettingsName", "Mac"),
				LOCTEXT("MacTargetSettingsDescription", "Settings for Mac target platform"),
				GetMutableDefault<UMacTargetSettings>()
			);
            SettingsModule->RegisterSettings("Project", "Platforms", "Xcode",
                LOCTEXT("XcodeProjectSettingsName", "Xcode Projects"),
                LOCTEXT("XcodeProjectSettingsDescription", "Settings for Xcode projects"),
                GetMutableDefault<UXcodeProjectSettings>()
            );
            
            FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
            PropertyModule.RegisterCustomClassLayout(
                "XcodeProjectSettings",
                FOnGetDetailCustomizationInstance::CreateStatic(&FXcodeProjectSettingsDetailsCustomization::MakeInstance)
            );
			
			PropertyModule.RegisterCustomClassLayout(
				"MacTargetSettings",
				FOnGetDetailCustomizationInstance::CreateStatic(&FMacTargetSettingsCustomization::MakeInstance)
			);
		}
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "Mac");
            SettingsModule->UnregisterSettings("Project", "Platforms", "Xcode");
		}
	}
};


IMPLEMENT_MODULE(FMacPlatformEditorModule, MacPlatformEditor);

#undef LOCTEXT_NAMESPACE
