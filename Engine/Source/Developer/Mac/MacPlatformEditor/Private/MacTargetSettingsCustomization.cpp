// Copyright Epic Games, Inc. All Rights Reserved.

#include "MacTargetSettingsCustomization.h"
#include "MacTargetSettings.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Modules/ModuleManager.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "EditorDirectories.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "SExternalImageReference.h"
#include "Widgets/Input/SNumericDropDown.h"
#include "Dialogs/Dialogs.h"
#include "Widgets/Notifications/SErrorText.h"
#include "IDetailPropertyRow.h"
#include "RHI.h"
#include "ShaderFormatsPropertyDetails.h"
#include "ISettingsEditorModule.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "ShaderPlatformConfigEditor.h"

namespace MacTargetSettingsDetailsConstants
{
	/** The filename for the game splash screen */
	const FString GameSplashFileName(TEXT("Splash/Splash.bmp"));

	/** The filename for the editor splash screen */
	const FString EditorSplashFileName(TEXT("Splash/EdSplash.bmp"));
}

#define LOCTEXT_NAMESPACE "MacTargetSettingsDetails"

TSharedRef<IDetailCustomization> FMacTargetSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FMacTargetSettingsCustomization);
}

namespace EMacImageScope
{
	enum Type
	{
		Engine,
		GameOverride
	};
}

/* Helper function used to generate filenames for splash screens */
static FString GetSplashFilename(EMacImageScope::Type Scope, bool bIsEditorSplash)
{
	FString Filename;

	if (Scope == EMacImageScope::Engine)
	{
		Filename = FPaths::EngineContentDir();
	}
	else
	{
		Filename = FPaths::ProjectContentDir();
	}

	if(bIsEditorSplash)
	{
		Filename /= MacTargetSettingsDetailsConstants::EditorSplashFileName;
	}
	else
	{
		Filename /= MacTargetSettingsDetailsConstants::GameSplashFileName;
	}

	Filename = FPaths::ConvertRelativePathToFull(Filename);

	return Filename;
}

/* Helper function used to generate filenames for icons */
static FString GetIconFilename(EMacImageScope::Type Scope)
{
	if (Scope == EMacImageScope::Engine)
	{
		FString Filename = FPaths::EngineDir() / FString(TEXT("Source/Runtime/Launch/Resources/Mac/UnrealEngine.icns"));
		return FPaths::ConvertRelativePathToFull(Filename);
	}
	else
	{
		FString Filename = FPaths::ProjectDir() / TEXT("Build/Mac/Application.icns");
		if(!FPaths::FileExists(Filename))
		{
			FString LegacyFilename = FPaths::GameSourceDir() / FString(FApp::GetProjectName()) / FString(TEXT("Resources/Mac")) / FString(FApp::GetProjectName()) + TEXT(".icns");
			if(FPaths::FileExists(LegacyFilename))
			{
				Filename = LegacyFilename;
			}
		}
		return FPaths::ConvertRelativePathToFull(Filename);
	}
}

static FText GetFriendlyNameFromRHINameMac(FName InRHIName)
{
	FText FriendlyRHIName;

	const EShaderPlatform Platform = ShaderFormatToLegacyShaderPlatform(InRHIName);
	switch (Platform)
	{
	case SP_PCD3D_SM5:
		FriendlyRHIName = LOCTEXT("D3DSM5", "Direct3D 11+ (SM5)");
		break;
	case SP_PCD3D_ES3_1:
		FriendlyRHIName = LOCTEXT("D3DESMobile", "Direct3D (Mobile Preview)");
		break;
	case SP_OPENGL_PCES3_1:
		FriendlyRHIName = LOCTEXT("OpenGLESMobilePC", "OpenGL (Mobile Preview)");
		break;
	case SP_OPENGL_ES3_1_ANDROID:
		FriendlyRHIName = LOCTEXT("OpenGLESMobile", "OpenGLES (Mobile)");
		break;
	case SP_METAL_ES3_1_IOS:
		FriendlyRHIName = LOCTEXT("Metal", "iOS Metal Mobile Renderer (Mobile, Metal 2.4+, iOS 15.0 or later)");
		break;
	case SP_METAL_SM5_IOS:
		FriendlyRHIName = LOCTEXT("MetalMobileSM5", "iOS Metal Desktop Renderer (SM5, Metal 2.4+, iOS 15.0 or later)");
		break;
	case SP_METAL_SM6_IOS:
		FriendlyRHIName = LOCTEXT("MetalMobileSM6", "iOS Metal Desktop Renderer (SM6, Metal 3.0+, iOS 18.0 or later)");
		break;
	case SP_METAL_ES3_1_TVOS:
		FriendlyRHIName = LOCTEXT("MetalTV", "tvOS Metal Mobile Renderer (Mobile, Metal 2.4+, tvOS 15.0 or later)");
		break;
	case SP_METAL_SM5_TVOS:
		FriendlyRHIName = LOCTEXT("MetalMobileSM5TV", "tvOS Metal Desktop Renderer (SM5, Metal 2.4+, tvOS 15.0 or later)");
		break;
	case SP_METAL_SM5:
		FriendlyRHIName = LOCTEXT("MetalSM5", "Mac Metal Desktop Renderer (SM5, Metal 2.4+, macOS Monterey 12.0 or later)");
		break;
	case SP_METAL_SM6:
		FriendlyRHIName = LOCTEXT("MetalSM6", "Mac Metal Desktop Renderer Beta (SM6, Metal 3.0+, macOS 15.0 or later, M2+)");
		break;
	case SP_METAL_SIM:
		FriendlyRHIName = LOCTEXT("MetalSim", "iOS Metal Simulator Mobile Renderer (Simulator, Metal 2.4+, iOS 15.0 or later)");
		break;
	case SP_METAL_ES3_1:
		FriendlyRHIName = LOCTEXT("MetalMobile", "Mac Metal High-End Mobile Preview (Mobile Preview)");
		break;
	case SP_VULKAN_SM5:
	case SP_VULKAN_SM5_ANDROID:
		FriendlyRHIName = LOCTEXT("VulkanSM5", "Vulkan (SM5)");
		break;
	case SP_VULKAN_SM6:
		FriendlyRHIName = LOCTEXT("VulkanSM6", "Vulkan (SM6)");
		break;
	case SP_VULKAN_PCES3_1:
	case SP_VULKAN_ES3_1_ANDROID:
		FriendlyRHIName = LOCTEXT("VulkanMobile", "Vulkan (Mobile)");
		break;
	default:
		FriendlyRHIName = FText::FromString(InRHIName.ToString());
		break;
	}

	return FriendlyRHIName;
}

void FMacTargetSettingsCustomization::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FSimpleDelegate OnUpdateShaderStandardWarning = FSimpleDelegate::CreateSP(this, &FMacTargetSettingsCustomization::UpdateShaderStandardWarning);
	
	ITargetPlatformSettings* TargetPlatformSettings = FModuleManager::GetModuleChecked<ITargetPlatformSettingsModule>("MacTargetPlatformSettings").GetTargetPlatformSettings()[0];

	// Setup the supported/targeted RHI property view
	TargetShaderFormatsDetails = MakeShareable(new FShaderFormatsPropertyDetails(&DetailBuilder, TEXT("TargetedRHIs"), TEXT("Targeted RHIs")));
	TargetShaderFormatsDetails->SetOnUpdateShaderWarning(OnUpdateShaderStandardWarning);
	TargetShaderFormatsDetails->CreateTargetShaderFormatsPropertyView(TargetPlatformSettings, &GetFriendlyNameFromRHINameMac, nullptr, ECategoryPriority::Important);

	UE::ShaderPlatformConfigEditor::AddShaderPlatformConfigs(DetailBuilder, TargetPlatformSettings);
	
	// Setup the shader version property view
	// Handle max. shader version a little specially.
	{
		IDetailCategoryBuilder& RenderCategory = DetailBuilder.EditCategory(TEXT("Rendering"));
		ShaderVersionPropertyHandle = DetailBuilder.GetProperty(TEXT("MetalLanguageVersion"));
		
		// Drop-downs for setting type of lower and upper bound normalization
		IDetailPropertyRow& ShaderVersionPropertyRow = RenderCategory.AddProperty(ShaderVersionPropertyHandle.ToSharedRef());
		ShaderVersionPropertyRow.CustomWidget()
		.NameContent()
		[
			ShaderVersionPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &FMacTargetSettingsCustomization::OnGetShaderVersionContent)
				.ContentPadding(FMargin( 2.0f, 2.0f ))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(this, &FMacTargetSettingsCustomization::GetShaderVersionDesc)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.Padding(2)
			[
				SAssignNew(ShaderVersionWarningTextBox, SErrorText)
				.AutoWrapText(true)
			]
		];
		
		UpdateShaderStandardWarning();
	}
	
	// Add the splash image customization
	const FText EditorSplashDesc(LOCTEXT("EditorSplashLabel", "Editor Splash"));
	IDetailCategoryBuilder& SplashCategoryBuilder = DetailBuilder.EditCategory(TEXT("Splash"));
	FDetailWidgetRow& EditorSplashWidgetRow = SplashCategoryBuilder.AddCustomRow(EditorSplashDesc);

	const FString EditorSplash_TargetImagePath = GetSplashFilename(EMacImageScope::GameOverride, true);
	const FString EditorSplash_DefaultImagePath = GetSplashFilename(EMacImageScope::Engine, true);

	EditorSplashWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(EditorSplashDesc)
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, EditorSplash_DefaultImagePath, EditorSplash_TargetImagePath)
			.FileDescription(EditorSplashDesc)
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FMacTargetSettingsCustomization::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FMacTargetSettingsCustomization::HandlePostExternalIconCopy))
		]
	];

	const FText GameSplashDesc(LOCTEXT("GameSplashLabel", "Game Splash"));
	FDetailWidgetRow& GameSplashWidgetRow = SplashCategoryBuilder.AddCustomRow(GameSplashDesc);

	const FString GameSplash_TargetImagePath = GetSplashFilename(EMacImageScope::GameOverride, false);
	const FString GameSplash_DefaultImagePath = GetSplashFilename(EMacImageScope::Engine, false);

	GameSplashWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(GameSplashDesc)
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, GameSplash_DefaultImagePath, GameSplash_TargetImagePath)
			.FileDescription(GameSplashDesc)
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FMacTargetSettingsCustomization::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FMacTargetSettingsCustomization::HandlePostExternalIconCopy))
		]
	];

	IDetailCategoryBuilder& IconsCategoryBuilder = DetailBuilder.EditCategory(TEXT("Icon"));	
	FDetailWidgetRow& GameIconWidgetRow = IconsCategoryBuilder.AddCustomRow(LOCTEXT("GameIconLabel", "Game Icon"));
	GameIconWidgetRow
	.NameContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding( FMargin( 0, 1, 0, 1 ) )
		.FillWidth(1.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("GameIconLabel", "Game Icon"))
			.Font(DetailBuilder.GetDetailFont())
		]
	]
	.ValueContent()
	.MaxDesiredWidth(500.0f)
	.MinDesiredWidth(100.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SExternalImageReference, GetIconFilename(EMacImageScope::Engine), GetIconFilename(EMacImageScope::GameOverride))
			.FileDescription(GameSplashDesc)
			.OnPreExternalImageCopy(FOnPreExternalImageCopy::CreateSP(this, &FMacTargetSettingsCustomization::HandlePreExternalIconCopy))
			.OnGetPickerPath(FOnGetPickerPath::CreateSP(this, &FMacTargetSettingsCustomization::GetPickerPath))
			.OnPostExternalImageCopy(FOnPostExternalImageCopy::CreateSP(this, &FMacTargetSettingsCustomization::HandlePostExternalIconCopy))
		]
	];

	AudioPluginWidgetManager.BuildAudioCategory(DetailBuilder, FString(TEXT("Mac")));
}


bool FMacTargetSettingsCustomization::HandlePreExternalIconCopy(const FString& InChosenImage)
{
	return true;
}


FString FMacTargetSettingsCustomization::GetPickerPath()
{
	return FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);
}


bool FMacTargetSettingsCustomization::HandlePostExternalIconCopy(const FString& InChosenImage)
{
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InChosenImage));
	return true;
}

static uint8 GMacTargetSettingsMinOSVers[][3] = {
	{12,0,0},
	{13,0,0}
};

TSharedRef<SWidget> FMacTargetSettingsCustomization::OnGetShaderVersionContent()
{
	FMenuBuilder MenuBuilder(true, NULL);
	
	UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/MacTargetPlatformSettings.EMacMetalShaderStandard"), EFindObjectFlags::ExactClass);
	
	for (int32 i = 0; i < Enum->GetMaxEnumValue(); i++)
	{
		bool bValidTargetForCurrentOS = true;
#if PLATFORM_MAC
		bValidTargetForCurrentOS = FPlatformMisc::MacOSXVersionCompare(GMacTargetSettingsMinOSVers[i][0], GMacTargetSettingsMinOSVers[i][1], GMacTargetSettingsMinOSVers[i][2]) >= 0;
#endif
		if (Enum->IsValidEnumValue(i) && bValidTargetForCurrentOS)
		{
			FUIAction ItemAction(FExecuteAction::CreateSP(this, &FMacTargetSettingsCustomization::SetShaderStandard, i));
			MenuBuilder.AddMenuEntry(Enum->GetDisplayNameTextByValue(i), TAttribute<FText>(), FSlateIcon(), ItemAction);
		}
	}
	
	return MenuBuilder.MakeWidget();
}

FText FMacTargetSettingsCustomization::GetShaderVersionDesc() const
{
	int32 EnumValue;
	ShaderVersionPropertyHandle->GetValue(EnumValue);
	
	UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/MacTargetPlatformSettings.EMacMetalShaderStandard"), EFindObjectFlags::ExactClass);
	
	if (EnumValue < Enum->GetMaxEnumValue() && Enum->IsValidEnumValue(EnumValue))
	{
		return Enum->GetDisplayNameTextByValue((uint8)EnumValue);
	}
	
	return FText::GetEmpty();
}

void FMacTargetSettingsCustomization::SetShaderStandard(int32 Value)
{
	if (ShaderVersionPropertyHandle->IsValidHandle())
	{
		FPropertyAccess::Result Res = ShaderVersionPropertyHandle->SetValue(Value);
		check(Res == FPropertyAccess::Success);
	}
	
	ShaderVersionWarningTextBox->SetError(TEXT(""));
	if (Value < 5 && Value != 0) // EMacMetalShaderStandard::MacMetalSLStandard_Minimum
	{
		FString EngineIdentifier = FEngineVersion::Current().ToString(EVersionComponent::Minor);
		
		ShaderVersionWarningTextBox->SetError(FString::Printf(TEXT("Minimum Metal Version is 2.2 in UE %s"), *EngineIdentifier));
	}
}

void FMacTargetSettingsCustomization::UpdateShaderStandardWarning()
{
	// Update the UI
	uint8 EnumValue = 0;

	if (ShaderVersionPropertyHandle->IsValidHandle())
	{
		ShaderVersionPropertyHandle->GetValue(EnumValue);
		if (EnumValue < 7 && EnumValue != 0)
		{
			SetShaderStandard(0); // EMacMetalShaderStandard::MacMetalSLStandard_Minimum
		}
	}
}

#undef LOCTEXT_NAMESPACE
