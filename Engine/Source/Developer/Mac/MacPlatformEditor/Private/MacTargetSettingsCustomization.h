// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "TargetPlatformAudioCustomization.h"
#include "Templates/SharedPointer.h"

class FShaderFormatsPropertyDetails;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SErrorText;
class SWidget;

class FMacTargetSettingsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	/** Delegate handler for before an icon is copied */
	bool HandlePreExternalIconCopy(const FString& InChosenImage);

	/** Delegate handler to get the path to start picking from */
	FString GetPickerPath();

	/** Delegate handler to set the path to start picking from */
	bool HandlePostExternalIconCopy(const FString& InChosenImage);
	
	/** Delegate handler to get the list of shader standards */
	TSharedRef<SWidget> OnGetShaderVersionContent();
	
	/** Delegate handler to get the description of the shader standard */
	FText GetShaderVersionDesc() const;
	
	/** Delegate handler to set the new max. shader standard */
	void SetShaderStandard(int32 Value);
	
	/** Delegate to update the shader standard warning */
	void UpdateShaderStandardWarning();
	
private:
	/** Reference to the target shader formats property view */
	TSharedPtr<FShaderFormatsPropertyDetails> TargetShaderFormatsDetails;
	
	/** Reference to the shader version property */
	TSharedPtr<IPropertyHandle> ShaderVersionPropertyHandle;
	
	/** Reference to the shader version property warning text box. */
	TSharedPtr< SErrorText > ShaderVersionWarningTextBox;

	/** Widget for platform specific audio plugins. */
	FAudioPluginWidgetManager AudioPluginWidgetManager;
};
