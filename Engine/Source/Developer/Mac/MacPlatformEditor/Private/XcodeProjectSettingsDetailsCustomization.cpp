// Copyright Epic Games, Inc. All Rights Reserved.

#include "XcodeProjectSettingsDetailsCustomization.h"
#include "XcodeProjectSettings.h"
#include "HAL/FileManager.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SButton.h"
#include "SourceControlHelpers.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "XcodeProjectSettings"

static constexpr FStringView kDefaultMacResourcesFolder(TEXTVIEW("Build/Mac/Resources/"));
static constexpr FStringView kDefaultIOSResourcesFolder(TEXTVIEW("Build/IOS/Resources/"));
static constexpr FStringView kDefaultIOSGeneratedFolder(TEXTVIEW("Build/IOS/UBTGenerated/"));

TSharedRef<IDetailCustomization> FXcodeProjectSettingsDetailsCustomization::MakeInstance()
{
    return MakeShareable(new FXcodeProjectSettingsDetailsCustomization);
}

void FXcodeProjectSettingsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
    TemplateMacPlist = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UXcodeProjectSettings, TemplateMacPlist));
    TemplateIOSPlist = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UXcodeProjectSettings, TemplateIOSPlist));
    PremadeMacEntitlements = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UXcodeProjectSettings, PremadeMacEntitlements));
    ShippingMacEntitlements = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UXcodeProjectSettings, ShippingSpecificMacEntitlements));
    PremadeIOSEntitlements = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UXcodeProjectSettings, PremadeIOSEntitlements));
    ShippingIOSEntitlements = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UXcodeProjectSettings, ShippingSpecificIOSEntitlements));

    IDetailCategoryBuilder& PlistCategory = DetailLayout.EditCategory(TEXT("Plist Files"));
    PlistCategory.AddCustomRow(LOCTEXT("InfoPlist", "Info.plist"), false)
    .WholeRowWidget
    [
        SNew(SHorizontalBox)
        + SHorizontalBox::Slot()
            .Padding(FMargin(0, 5, 0, 10))
            .AutoWidth()
        [
           SNew(SButton)
           .Text(LOCTEXT("RestoreInfoPlist", "Restore Info.plist to default"))
           .ToolTipText(LOCTEXT("RestoreInfoPlistTooltip", "Revert to use default templates copied from Engine"))
           .OnClicked(this, &FXcodeProjectSettingsDetailsCustomization::OnRestorePlistClicked)
         ]
    ];
    
    IDetailCategoryBuilder& ShipEntitlementCategory = DetailLayout.EditCategory(TEXT("Entitlements"));
    ShipEntitlementCategory.AddCustomRow(LOCTEXT("Entitlement", "Entitlement"), false)
    .WholeRowWidget
    [
        SNew(SHorizontalBox)
        + SHorizontalBox::Slot()
            .Padding(FMargin(0, 5, 0, 10))
            .AutoWidth()
        [
            SNew(SButton)
            .Text(LOCTEXT("RestoreEntitlements", "Restore entitlements to default"))
            .ToolTipText(LOCTEXT("RestoreEntitlementsTooltip", "Revert to use default entitlements copied from Engine"))
            .OnClicked(this, &FXcodeProjectSettingsDetailsCustomization::OnRestoreEntitlementClicked)
        ]
    ];
    
    DetailLayout.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
    {
        for (const TPair<FName, IDetailCategoryBuilder*>& Pair : CategoryMap)
        {
            int32 SortOrder = Pair.Value->GetSortOrder();
            const FName CategoryName = Pair.Key;

            if (CategoryName == "Xcode")
            {
                SortOrder = 0;
            }
            else if(CategoryName == "Plist Files")
            {
                SortOrder = 1;
            }
			else if(CategoryName == "Entitlements")
			{
				SortOrder = 2;
			}
			else if(CategoryName == "Code Signing")
			{
				SortOrder = 3;
			}
			else if(CategoryName == "Privacy Manifests")
			{
				SortOrder = 4;
			}
            else
            {
                // Unknown category, should explicitly set order
                ensureMsgf(false, TEXT("Unknown category %s in XcodeProjectSttings"), *CategoryName.ToString());
                SortOrder = 999;
            }

            Pair.Value->SetSortOrder(SortOrder);
        }

        return;
    });
}

void FXcodeProjectSettingsDetailsCustomization::RestoreDefault(const FStringView& SubFolder, const TCHAR* Filename, TSharedPtr<IPropertyHandle> Property)
{
	FText ErrorMessage;
	// Copy the default from Engine
	if (!SourceControlHelpers::CopyFileUnderSourceControl(FPaths::ProjectDir() + SubFolder + Filename,
														  FPaths::EngineDir() + SubFolder + Filename,
														  FText::FromString(Filename),
														  /*out*/ ErrorMessage))
	{
		FNotificationInfo Info(ErrorMessage);
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
	
	Property->SetValueFromFormattedString(TEXT("(FilePath=\"/Game/") + (FString)SubFolder + Filename + TEXT("\")"));
}


FReply FXcodeProjectSettingsDetailsCustomization::OnRestorePlistClicked()
{    
	RestoreDefault(kDefaultMacResourcesFolder, TEXT("Info.Template.plist"), TemplateMacPlist);
    
    // No need to copy iOS template, it uses generated plist
    TemplateIOSPlist->SetValueFromFormattedString(TEXT("(FilePath=\"/Game/") + (FString)kDefaultIOSGeneratedFolder + TEXT("Info.Template.plist\")"));

    return FReply::Handled();
}

FReply FXcodeProjectSettingsDetailsCustomization::OnRestoreEntitlementClicked()
{
    const UXcodeProjectSettings* Settings = GetDefault<UXcodeProjectSettings>();
    
    RestoreDefault(kDefaultMacResourcesFolder, TEXT("Sandbox.Server.entitlements"), PremadeMacEntitlements);
	RestoreDefault(kDefaultMacResourcesFolder, TEXT("Sandbox.NoNet.entitlements"), ShippingMacEntitlements);
	
	PremadeIOSEntitlements->SetValueFromFormattedString(TEXT("(FilePath=\"\")"));
	ShippingIOSEntitlements->SetValueFromFormattedString(TEXT("(FilePath=\"\")"));
	
    return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
