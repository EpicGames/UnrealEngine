// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 XcodeProjectSettingsDetailsCustomization.h: Declares the FXcodeProjectSettingsDetailsCustomization class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "IDetailCustomization.h"

class IPropertyHandle;
class FReply;

class FXcodeProjectSettingsDetailsCustomization : public IDetailCustomization
{
public:
    /** Makes a new instance of this detail layout class for a specific detail view requesting it */
    static TSharedRef<IDetailCustomization> MakeInstance();

    // IDetailCustomization interface
    virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
    // End of IDetailCustomization interface
    
    FReply OnRestorePlistClicked();
    FReply OnRestoreEntitlementClicked();
    
    TSharedPtr<IPropertyHandle> TemplateMacPlist;
    TSharedPtr<IPropertyHandle> TemplateIOSPlist;
    TSharedPtr<IPropertyHandle> PremadeMacEntitlements;
    TSharedPtr<IPropertyHandle> ShippingMacEntitlements;
    TSharedPtr<IPropertyHandle> PremadeIOSEntitlements;
    TSharedPtr<IPropertyHandle> ShippingIOSEntitlements;
	
private:
	static void RestoreDefault(const FStringView& SubFolder, const TCHAR* Filename, TSharedPtr<IPropertyHandle> Property);
};
