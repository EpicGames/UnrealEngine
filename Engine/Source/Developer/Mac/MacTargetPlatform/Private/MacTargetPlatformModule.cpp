// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Common/TargetPlatformBase.h"

#define LOCTEXT_NAMESPACE "FMacTargetPlatformModule"

/**
 * Module for Mac as a target platform
 */
class FMacTargetPlatformModule
	: public ITargetPlatformModule
{
public:

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms) override
	{
	}

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls) override
	{
		for (ITargetPlatformControls* TargetPlatformControlsIt : TargetPlatformControls)
		{
			TargetPlatforms.Add(new FTargetPlatformMerged(TargetPlatformControlsIt->GetTargetPlatformSettings(), TargetPlatformControlsIt));
		}
	}

public:

	// Begin IModuleInterface interface

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

	// End IModuleInterface interface
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FMacTargetPlatformModule, MacTargetPlatform);
