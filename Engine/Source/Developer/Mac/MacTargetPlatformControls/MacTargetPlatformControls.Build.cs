// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MacTargetPlatformControls : ModuleRules
{
	public MacTargetPlatformControls(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.Mac);

		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "MacTPCon";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform",
				"MacTargetPlatformSettings"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings"
			}
		);

		if (Target.bCompileAgainstEngine)
		{
            PublicIncludePathModuleNames.Add("Engine");
            PrivateDependencyModuleNames.AddRange(
                new string[] {
                    "Engine",
                    "CookedEditor",
                }
            );
			PrivateIncludePathModuleNames.Add("TextureCompressor");
		}
	}
}
