// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericMacTargetPlatformControls.h"
#include "LocalMacTargetDevice.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Modules/ModuleManager.h"
#include "IMacTargetPlatformSettingsModule.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if WITH_ENGINE
#include "CookedEditorTargetPlatformControls.h"
#endif
#define LOCTEXT_NAMESPACE "FMacTargetPlatformControlsModule"

/**
 * Module for Mac as a target platform controls
 */
class FMacTargetPlatformControlsModule
	: public ITargetPlatformControlsModule
{
public:
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) override
	{
		TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;
		IMacTargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<IMacTargetPlatformSettingsModule>(PlatformSettingsModuleName);
		if (ModuleSettings != nullptr)
		{
			TMap<FString, ITargetPlatformSettings*> OutMap;
			ModuleSettings->GetPlatformSettingsMaps(OutMap);
			ITargetPlatformControls* GameTP = new TGenericMacTargetPlatformControls<false, false, false>(OutMap[FMacPlatformProperties<false, false, false>::PlatformName()]);
			ITargetPlatformControls* EditorTP = new TGenericMacTargetPlatformControls<true, false, false>(OutMap[FMacPlatformProperties<true, false, false>::PlatformName()]);
			ITargetPlatformControls* ServerTP = new TGenericMacTargetPlatformControls<false, true, false>(OutMap[FMacPlatformProperties<false, true, false>::PlatformName()]);
			ITargetPlatformControls* ClientTP = new TGenericMacTargetPlatformControls<false, false, true>(OutMap[FMacPlatformProperties<false, false, true>::PlatformName()]);
			TargetPlatforms.Add(GameTP);
			TargetPlatforms.Add(EditorTP);
			TargetPlatforms.Add(ServerTP);
			TargetPlatforms.Add(ClientTP);
#if WITH_ENGINE
			// currently this TP requires the engine for allowing GameDelegates usage
			bool bSupportCookedEditor;
			if (GConfig->GetBool(TEXT("CookedEditorSettings"), TEXT("bSupportCookedEditor"), bSupportCookedEditor, GGameIni) && bSupportCookedEditor)
			{
				ITargetPlatformControls* CookedEditorTP = new TCookedEditorTargetPlatformControls<FMacEditorTargetPlatformControlsParent>(ModuleSettings->GetCookedEditorPlatformSettings());
				ITargetPlatformControls* CookedCookerTP = new TCookedCookerTargetPlatformControls<FMacEditorTargetPlatformControlsParent>(ModuleSettings->GetCookedCookerPlatformSettings());
				TargetPlatforms.Add(CookedEditorTP);
				TargetPlatforms.Add(CookedCookerTP);
			}
#endif
		}
	}

public:

	// Begin IModuleInterface interface

	virtual void StartupModule() override
	{

	}

	virtual void ShutdownModule() override
	{

	}

	// End IModuleInterface interface

};

bool FLocalMacTargetDevice::Reboot( bool bReconnect )
{
#if PLATFORM_MAC
	NSAppleScript* Script = [[NSAppleScript alloc] initWithSource:@"tell application \"System Events\" to restart"];
	NSDictionary* ErrorDict = [NSDictionary dictionary];
	[Script executeAndReturnError: &ErrorDict];
#endif
	return true;
}


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FMacTargetPlatformControlsModule, MacTargetPlatformControls);
