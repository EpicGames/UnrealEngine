// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenericMacTargetPlatform.h: Declares the TGenericMacTargetPlatformControls class template.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Common/TargetPlatformControlsBase.h"
#include "Mac/MacPlatformProperties.h"
#include "LocalMacTargetDevice.h"
#include "AnalyticsEventAttribute.h"
#if WITH_ENGINE
#include "StaticMeshResources.h"
#include "TextureResource.h"
#endif
#define LOCTEXT_NAMESPACE "TGenericMacTargetPlatformControls"

/**
 * Template for Mac target platforms controls
 */
template<bool HAS_EDITOR_DATA, bool IS_DEDICATED_SERVER, bool IS_CLIENT_ONLY>
class TGenericMacTargetPlatformControls
	: public TTargetPlatformControlsBase<FMacPlatformProperties<HAS_EDITOR_DATA, IS_DEDICATED_SERVER, IS_CLIENT_ONLY> >
{
public:

	typedef FMacPlatformProperties<HAS_EDITOR_DATA, IS_DEDICATED_SERVER, IS_CLIENT_ONLY> TProperties;
	typedef TTargetPlatformControlsBase<TProperties> TSuper;

	/**
	 * Default constructor.
	 */
	TGenericMacTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings)
		: TSuper(TargetPlatformSettings)
	{
#if PLATFORM_MAC
		// only add local device if actually running on Mac
		LocalDevice = MakeShareable(new FLocalMacTargetDevice(*this));
#endif
	}

public:

	//~ Begin ITargetPlatform Interface

	virtual void EnableDeviceCheck(bool OnOff) override {}

	virtual void GetAllDevices( TArray<ITargetDevicePtr>& OutDevices ) const override
	{
		OutDevices.Reset();
		if (LocalDevice.IsValid())
		{
			OutDevices.Add(LocalDevice);
		}
	}

	virtual bool GenerateStreamingInstallManifest(const TMultiMap<FString, int32>& PakchunkMap, const TSet<int32>& PakchunkIndicesInUse) const override
	{
		return true;
	}

	virtual ITargetDevicePtr GetDefaultDevice( ) const override
	{
		if (LocalDevice.IsValid())
		{
			return LocalDevice;
		}

		return nullptr;
	}

	virtual ITargetDevicePtr GetDevice( const FTargetDeviceId& DeviceId )
	{
		if (LocalDevice.IsValid() && (DeviceId == LocalDevice->GetId()))
		{
			return LocalDevice;
		}

		return NULL;
	}


	virtual void GetPlatformSpecificProjectAnalytics( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray ) const override
	{
		TSuper::GetPlatformSpecificProjectAnalytics(AnalyticsParamArray);

		TSuper::AppendAnalyticsEventConfigArray(AnalyticsParamArray, TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("TargetedRHIs"), GEngineIni);
	}

#if WITH_ENGINE

	virtual void GetTextureFormats( const UTexture* Texture, TArray< TArray<FName> >& OutFormats) const override
	{
		if (!IS_DEDICATED_SERVER)
		{
			// just use the standard texture format name for this texture (with DX11 support)
			GetDefaultTextureFormatNamePerLayer(OutFormats.AddDefaulted_GetRef(), this->GetTargetPlatformSettings(), this, Texture, true, 4, true);
		}
	}

	virtual bool SupportsLQCompressionTextureFormat() const override { return false; };

	virtual bool CanSupportRemoteShaderCompile() const override
	{
		return true;
	}
	
	virtual void GetShaderCompilerDependencies(TArray<FString>& OutDependencies) const override
	{
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/Mac/libdxcompiler.dylib"));
		FTargetPlatformControlsBase::AddDependencySCArrayHelper(OutDependencies, TEXT("Binaries/ThirdParty/ShaderConductor/Mac/libShaderConductor.dylib"));
	}

#endif //WITH_ENGINE


	virtual bool SupportsVariants() const override
	{
		return true;
	}


	virtual float GetVariantPriority() const override
	{
		return TProperties::GetVariantPriority();
	}

	//~ End ITargetPlatform Interface

private:

	// Holds the local device.
	ITargetDevicePtr LocalDevice;

#if WITH_ENGINE
	// Holds the texture LOD settings.
	const UTextureLODSettings* TextureLODSettings;

	// Holds the static mesh LOD settings.
	FStaticMeshLODSettings StaticMeshLODSettings;

#endif // WITH_ENGINE

};

#undef LOCTEXT_NAMESPACE
