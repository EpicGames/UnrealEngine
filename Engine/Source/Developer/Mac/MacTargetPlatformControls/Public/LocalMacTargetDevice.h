// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LocalMacTargetDevice.h: Declares the TLocalMacTargetDevice class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/ITargetDevice.h"
#include "Interfaces/ITargetPlatformControls.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"

/**
 * Template for local Mac target devices.
 */
class FLocalMacTargetDevice
	: public ITargetDevice
{
public:

	/**
	 * Creates and initializes a new device for the specified target platform.
	 *
	 * @param InTargetPlatform - The target platform.
	 */
	FLocalMacTargetDevice(const ITargetPlatformControls& InTargetPlatformControls)
		: TargetPlatformControls(InTargetPlatformControls)
	{ }


public:

	virtual bool Connect( ) override
	{
		return true;
	}

	virtual void Disconnect( )
	{ }

	virtual ETargetDeviceTypes GetDeviceType( ) const override
	{
		return ETargetDeviceTypes::Desktop;
	}

	virtual FTargetDeviceId GetId( ) const override
	{
		return FTargetDeviceId(TargetPlatformControls.PlatformName(), GetName());
	}

	virtual FString GetName( ) const override
	{
		return FPlatformProcess::ComputerName();
	}

	virtual FString GetOperatingSystemName( ) override
	{
		return TEXT("macOS");
	}

	virtual int32 GetProcessSnapshot( TArray<FTargetDeviceProcessInfo>& OutProcessInfos ) override
	{
		// @todo Mac: implement process snapshots
		return 0;
	}

	virtual const class ITargetPlatformSettings& GetPlatformSettings() const override
	{
		return *(TargetPlatformControls.GetTargetPlatformSettings());
	}
	virtual const class ITargetPlatformControls& GetPlatformControls() const override
	{
		return TargetPlatformControls;
	}

	virtual bool IsConnected( )
	{
		return true;
	}

	virtual bool IsDefault( ) const override
	{
		return true;
	}

	virtual bool PowerOff( bool Force ) override
	{
		return false;
	}

	virtual bool PowerOn( ) override
	{
		return false;
	}

	MACTARGETPLATFORMCONTROLS_API virtual bool Reboot( bool bReconnect = false ) override;

	virtual bool SupportsFeature( ETargetDeviceFeatures Feature ) const override
	{
		switch (Feature)
		{
		case ETargetDeviceFeatures::MultiLaunch:
			return true;

		// @todo Mac: implement process snapshots
		case ETargetDeviceFeatures::ProcessSnapshot:
			return false;

		case ETargetDeviceFeatures::Reboot:
			return true;
		}

		return false;
	}

	virtual void SetUserCredentials( const FString& UserName, const FString& UserPassword ) override
	{
	}

	virtual bool GetUserCredentials( FString& OutUserName, FString& OutUserPassword ) override
	{
		return false;
	}

	virtual bool TerminateProcess( const int64 ProcessId ) override
	{
		return false;
	}


private:

	// Holds a reference to the device's target platform.
	const ITargetPlatformControls& TargetPlatformControls;
};
