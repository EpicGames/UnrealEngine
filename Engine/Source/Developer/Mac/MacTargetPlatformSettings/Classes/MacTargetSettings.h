// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MacTargetSettings.h: Declares the UMacTargetSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "MacTargetSettings.generated.h"

UENUM()
enum class EMacMetalShaderStandard : uint8
{
    /** Metal Shader 2.2 is the minimum*/
    MacMetalSLStandard_Minimum = 0 UMETA(DisplayName="Minimum, Currently v2.2 (10.15+)"),
    /** Metal Shaders Compatible With macOS 10.15 or later (std=osx-metal2.2) */
    MacMetalSLStandard_2_2 = 5 UMETA(DisplayName="Metal v2.2 (10.15+)"),
    /** Metal Shaders Compatible With macOS 11.0 or later (std=osx-metal2.3) */
    MacMetalSLStandard_2_3 = 6 UMETA(DisplayName="Metal v2.3 (11.0+)"),
    /** Metal Shaders Compatible With macOS 12.0 or later (std=osx-metal2.4) */
    MacMetalSLStandard_2_4 = 7 UMETA(DisplayName="Metal v2.4 (12.0+)"),
    /** Metal Shaders Compatible With macOS 13.0 or later (std=metal3.0) */
    MacMetalSLStandard_3_0 = 8 UMETA(DisplayName="Metal v3.0 (13.0+)"),

};

UENUM()
enum class EMacTargetArchitecture : uint8
{
    /** Create packages that can run natively on Intel Macs and under translation on Apple Silicon Macs */
    MacTargetArchitectureIntel = 0 UMETA(DisplayName="Intel x64"),
    
    /** Create Universal packages that run natively on all Macs */
    MacTargetArchitectureUniversal = 1 UMETA(DisplayName="Universal (Intel & Apple Silicon)"),

	/** Create packages that can run natively on Apple Silicon Macs, and will not run on Intel Macs */
	MacTargetArchitectureAppleSilicon = 2 UMETA(DisplayName="Apple Silicon"),

	/** Create  packages that match the architecture of the Mac the package is made on */
	MacTargetArchitectureHost = 3 UMETA(DisplayName="Host (Matches current machine)"),
};


/**
 * Implements the settings for the Mac target platform.
 */
UCLASS(config=Engine, defaultconfig)
class MACTARGETPLATFORMSETTINGS_API UMacTargetSettings
	: public UObject
{
public:

	GENERATED_UCLASS_BODY()
	
	/**
	 * The collection of RHI's we want to support on this platform.
	 * This is not always the full list of RHI we can support.
	 */
	UPROPERTY(EditAnywhere, config, Category=Rendering)
	TArray<FString> TargetedRHIs;
	
	/**
	 * The set of architecture(s) this project supports for Editor builds
	 * This defines which CPU architectures to target: x86_64 (Intel), arm64 (Apple Silicon) or Universal (Intel & Apple Silicon).
	 * It is recommended to use Universal unless you have editor plugins or other libraries that do not support Apple Silicon
	 */
	UPROPERTY(EditAnywhere, config, Category=Packaging, meta = (DisplayName = "Supported Architecture(s) for Editor builds"))
	EMacTargetArchitecture EditorTargetArchitecture;
	
	/**
	 * The target Mac platform CPU architecture.
	 * This defines which CPU architectures to target: x86_64 (Intel), arm64 (Apple Silicon) or Universal (Intel & Apple Silicon).
	 * It is recommended to use Universal unless you have runtime plugins or other libraries that do not support Apple Silicon
	 */
	UPROPERTY(EditAnywhere, config, Category=Packaging, meta = (DisplayName = "Supported Architecture(s) for non-Editor builds"))
	EMacTargetArchitecture TargetArchitecture;
	
	/**
	 * The architecture to compile the Editor target
	 * This defines which CPU architectures to target: x86_64 (Intel), arm64 (Apple Silicon) or Universal (Intel & Apple Silicon), or Host to match the machine doing the building
	 * Can override with -Architecture= on the UBT commandline, or -EditorArchitecture=  on the BuildCookRun commandline
	 */
	UPROPERTY(EditAnywhere, config, Category=Packaging, meta = (DisplayName = "Architecture(s) when building Editor"))
	EMacTargetArchitecture EditorDefaultArchitecture;
	
	/**
	 * The architectures to compile non-Editor (games, programs, etc) targets for builds outside of Xcode
	 * This defines which CPU architectures to target: x86_64 (Intel), arm64 (Apple Silicon) or Universal (Intel & Apple Silicon), or Host to match the machine doing the building
	 * Can override with -Architecture= on the UBT commandline, or -GameArchitecture= or -ProgramArchitecture= on the BuildCookRun commandline
	 */
	UPROPERTY(EditAnywhere, config, Category=Packaging, meta = (DisplayName = "Architecture(s) when building non-Editor"))
	EMacTargetArchitecture DefaultArchitecture;
	
	/**
	 * If true, builds running on BuildMachines (when the 'IsBuildMachine' environment variable is set to 1) will compile all Supported architectures
	 */
	UPROPERTY(EditAnywhere, config, Category=Packaging, meta = (DisplayName = "Build all supported Architectures on Build Machines"))
	bool bBuildAllSupportedOnBuildMachine;

    /**
     * The Metal shader language version which will be used when compiling the shaders.
     */
    UPROPERTY(EditAnywhere, config, Category=Rendering, meta = (DisplayName = "Metal Shader Standard To Target", ConfigRestartRequired = true))
    int32 MetalLanguageVersion;
    
    /**
     * Whether to use the Metal shading language's "fast" intrinsics.
	 * Fast intrinsics assume that no NaN or INF value will be provided as input, 
	 * so are more efficient. However, they will produce undefined results if NaN/INF 
	 * is present in the argument/s. By default fast-instrinics are disabled so Metal correctly handles NaN/INF arguments.
     */
    UPROPERTY(EditAnywhere, config, Category=Rendering, meta = (DisplayName = "Use Fast-Math intrinsics", ConfigRestartRequired = true))
	bool UseFastIntrinsics;
	
	/**
	 * Whether to use of Metal shader-compiler's -ffast-math optimisations.
	 * Fast-Math performs algebraic-equivalent & reassociative optimisations not permitted by the floating point arithmetic standard (IEEE-754).
	 * These can improve shader performance at some cost to precision and can lead to NaN/INF propagation as they rely on
	 * shader inputs or variables not containing NaN/INF values. By default fast-math is enabled for performance.
	 */
	UPROPERTY(EditAnywhere, config, Category=Rendering, meta = (DisplayName = "Enable Fast-Math optimisations", ConfigRestartRequired = true))
	bool EnableMathOptimisations;

	/** Sample rate to run the audio mixer with. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", Meta = (DisplayName = "Audio Mixer Sample Rate"))
	int32 AudioSampleRate;

	/** The amount of audio to compute each callback block. Lower values decrease latency but may increase CPU cost. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "512", ClampMax = "4096", DisplayName = "Callback Buffer Size"))
	int32 AudioCallbackBufferFrameSize;

	/** The number of buffers to keep enqueued. More buffers increases latency, but can compensate for variable compute availability in audio callbacks on some platforms. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "1", UIMin = "1", DisplayName = "Number of Buffers To Enqueue"))
	int32 AudioNumBuffersToEnqueue;

	/** The max number of channels (voices) to limit for this platform. The max channels used will be the minimum of this value and the global audio quality settings. A value of 0 will not apply a platform channel count max. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Max Channels"))
	int32 AudioMaxChannels;

	/** The number of workers to use to compute source audio. Will only use up to the max number of sources. Will evenly divide sources to each source worker. */
	UPROPERTY(config, EditAnywhere, Category = "Audio", meta = (ClampMin = "0", UIMin = "0", DisplayName = "Number of Source Workers"))
	int32 AudioNumSourceWorkers;

	/** Which of the currently enabled spatialization plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SpatializationPlugin;

	/** Which of the currently enabled source data override plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString SourceDataOverridePlugin;

	/** Which of the currently enabled reverb plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString ReverbPlugin;

	/** Which of the currently enabled occlusion plugins to use. */
	UPROPERTY(config, EditAnywhere, Category = "Audio")
	FString OcclusionPlugin;
	
	/** Quality Level to COOK SoundCues at (if set, all other levels will be stripped by the cooker). */
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Audio|CookOverrides", meta = (DisplayName = "Sound Cue Cook Quality"))
	int32 SoundCueCookQualityIndex = INDEX_NONE;

	void virtual OverrideConfigSection(FString& InOutSectionName) override
	{
		InOutSectionName = TEXT("/Script/MacTargetPlatform.MacTargetSettings");
	}
	
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override
	{
		if(TargetedRHIs.Num() == 0)
		{		
			TargetedRHIs.Add(TEXT("SF_METAL_SM5"));
		}
	}
#endif
};
