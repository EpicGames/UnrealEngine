// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
    XcodeProjectSettings.h: Declares the UXcodeProjectSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "XcodeProjectSettings.generated.h"

/**
 * Implements the settings for Xcode projects
 */
UCLASS(config=Engine, defaultconfig)
class MACTARGETPLATFORMSETTINGS_API UXcodeProjectSettings
    : public UObject
{
public:
	
	GENERATED_UCLASS_BODY()
	
	/**
	 * Enable modernized Xcode, when building from Xcode, use native Xcode for bundle generation and archiving instead of UBT
     * Restart required to apply this setting
	 */
	UE_DEPRECATED(5.8, "Modern Xcode is now the only option. This property is no longer used.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Modern Xcode is now the only option. This property is no longer used."))
	bool bUseModernXcode = true;
    
    UFUNCTION()
    static bool ShouldDisableIOSSettings()
    {
#if PLATFORM_MAC
        return true;
#else
        // On PC, even after turning on Modern Xcode for remote Mac, should let user config local iOS related settings
        return false;
#endif
    }
	
	/**
	 * Team ID used for native Xcode code signing. This must be the 10 letters/numbers ID found in Membership Details tab found in https://developer.apple.com/account
	 */
	UPROPERTY(EditAnywhere, config, Category=Xcode, DisplayName = "Apple Dev Account Team ID")
	FString CodeSigningTeam;
	
	/**
	 * Bundle ID used for nativr Xcode code signing
	 */
	UPROPERTY(EditAnywhere, config, Category=Xcode, DisplayName = "Bundle ID")
	FString BundleIdentifier;
	
	/**
	 * Bundle ID prefix used for native Xcode code signing. This is only needed if you use the default, pieced-together Bundle ID above. If you specify a full Bundle ID, you can ignore this field.
	 */
	UPROPERTY(EditAnywhere, config, Category=Xcode, DisplayName = "Bundle ID Prefix")
	FString CodeSigningPrefix;
	
	/**
	 * The name of the Mac .app when making an archived build (for uploading to App Store, etc). The Finder shows Mac apps by their .app name, and we don't name the .app  with
	 * "pretty" names during development. When packaging for distribution (or using Archive menu in Xcode) this will become the name of the .app, and will be what end users
	 * will have on their Mac. If this is not set, the .app will have the name of the .uproject file
	 */
	UPROPERTY(EditAnywhere, config, Category=Xcode, DisplayName = "Mac: Published App Name")
	FString ApplicationDisplayName;

	/**
	 * The App Category that will be used for Apple App Store submissions
	 */
	UPROPERTY(EditAnywhere, config, Category=Xcode, DisplayName = "App Category")
	FString AppCategory;
	
	/**
	 * The template info.plist used for Mac game targets
	 */
	UPROPERTY(EditAnywhere, config, Category="Plist Files", DisplayName = "Mac: Info.plist Template")
	FFilePath TemplateMacPlist;
	
	/**
	 * The template info.plist used for iOS game targets
	 */
	UPROPERTY(EditAnywhere, config, Category="Plist Files", DisplayName = "IOS / TVOS: Info.plist Template")
	FFilePath TemplateIOSPlist;
	
	/**
	 * The premade entitlement file used for development Mac builds
	 */
	UPROPERTY(EditAnywhere, config, Category="Entitlements", DisplayName = "Mac: Entitlements")
	FFilePath PremadeMacEntitlements;
	
	/**
	 * The premade entitlement file used for shipping Mac builds
	 */
	UPROPERTY(EditAnywhere, config, Category="Entitlements", DisplayName = "Mac:  Entitlements (Shipping config override)")
	FFilePath ShippingSpecificMacEntitlements;
	
	/**
	 * The premade entitlement file used for development IOS builds
	 */
	UPROPERTY(EditAnywhere, config, Category="Entitlements", DisplayName = "IOS: Entitlements")
	FFilePath PremadeIOSEntitlements;
	
	/**
	 * The premade entitlement file used for shipping IOS builds
	 */
	UPROPERTY(EditAnywhere, config, Category="Entitlements",  DisplayName = "IOS: Entitlements (Shipping config override)")
	FFilePath ShippingSpecificIOSEntitlements;
	
	/**
	 * Enable native Xcode code signing
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing")
	bool bUseAutomaticCodeSigning;
	
	/**
	 * If true, Mac will sign to run locally. Running on another Mac may bring up a dialog preventing running the app. If this and Use Automatic Code Signing are both false, you will need a certificate installed
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", DisplayName="Mac: Sign To Run Locally")
	bool bMacSignToRunLocally;
	
	/**
	 * The name (prefix or full) of the certificate to use for Mac code signing.
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="!bUseAutomaticCodeSigning && !bMacSignToRunLocally"), DisplayName = "Mac: Signing Identity")
	FString MacSigningIdentity;
	
	/**
	 * The name (prefix or full) of the certificate to use for IOS and TVOS code signing
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="!bUseAutomaticCodeSigning", DisplayName = "IOS / TVOS: Signing Identity"))
	FString IOSSigningIdentity;
	
	/**
	 * The path to a .mobileprovision file to use for signing for IOS. Alternatively, if it's a single name or UUID (no .mobileprovision extension), it will use this as the name/UUID of an already installed provision to sign with
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="!bUseAutomaticCodeSigning", DisplayName = "IOS: Provisioning Profile"))
	FFilePath IOSProvisioningProfile;
	
	/**
	 * The path to a .mobileprovision file to use for signing for TVOS. Alternatively, if it's a single name or UUID (no .mobileprovision extension), it will use this as the name/UUID of an already installed provision to sign with
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="!bUseAutomaticCodeSigning", DisplayName = "TVOS: Provisioning Profile"))
	FFilePath TVOSProvisioningProfile;
	
	/**
	 * If true, use AppStore Connect authentication for commandline builds. This allows for automatic codesigning functionality without needing to be signed in to Xcode. See the App Store Connect API section of the Keys tab in your Users and Access page in Apple dev center.
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="bUseAutomaticCodeSigning"))
	bool bUseAppStoreConnect;
	
	/**
	 * The Issuer ID for your App Store Connect API
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="bUseAutomaticCodeSigning && bUseAppStoreConnect", DisplayName = "App Store Connect: Issuer ID"))
	FString AppStoreConnectIssuerID;
	
	/**
	 * The Key ID for your App Store Connect generated API key, a 32 hex-character string, including dashes
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="bUseAutomaticCodeSigning && bUseAppStoreConnect", DisplayName = "App Store Connect: Key ID"))
	FString AppStoreConnectKeyID;
	
	/**
	 * The path to the downloaded .p8 file shared with your team
	 */
	UPROPERTY(EditAnywhere, config, Category="Code Signing", meta = (EditCondition="bUseAutomaticCodeSigning && bUseAppStoreConnect", DisplayName = "App Store Connect: Key File"))
	FFilePath AppStoreConnectKeyPath;
	
	/**
	 * The path to the optional PrivacyInfo.xcprivacy file for your project
	 */
	UPROPERTY(EditAnywhere, config, Category="Privacy Manifests", meta = (DisplayName = "Mac: Additional Privacy Info"))
	FFilePath AdditionalPrivacyInfoMac;
	
	/**
	 * The path to the optional PrivacyInfo.xcprivacy file for your project
	 */
	UPROPERTY(EditAnywhere, config, Category="Privacy Manifests", meta = (DisplayName = "IOS/TVOS/VisionOS: Additional Privacy Info"))
	FFilePath AdditionalPrivacyInfoIOS;

	void virtual OverrideConfigSection(FString& InOutSectionName) override
	{
		InOutSectionName = TEXT("/Script/MacTargetPlatform.XcodeProjectSettings");
	}
};
