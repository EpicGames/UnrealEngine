// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MacTargetPlatformSettings : ModuleRules
{
	public MacTargetPlatformSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "MacTPSet";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"TargetPlatform",
				"DesktopPlatform"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings"
			}
		);

		if (Target.bCompileAgainstEngine)
		{  
            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "CookedEditor",
					"RHI",
                }
            );
            PrivateDependencyModuleNames.Add("Engine");
			PrivateIncludePathModuleNames.Add("TextureCompressor");
		}
	}
}
