// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MacTargetPlatformClasses.cpp: Implements the module's UClasses.
=============================================================================*/

#include "MacTargetSettings.h"
#include "XcodeProjectSettings.h"


/* UMacTargetSettings structors
 *****************************************************************************/

UMacTargetSettings::UMacTargetSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }

/* UXcodeProjectSettings structors
 *****************************************************************************/

UXcodeProjectSettings::UXcodeProjectSettings( const FObjectInitializer& ObjectInitializer )
    : Super(ObjectInitializer)
{ }
