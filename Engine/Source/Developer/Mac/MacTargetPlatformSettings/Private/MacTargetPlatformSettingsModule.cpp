// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericMacTargetPlatformSettings.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Modules/ModuleManager.h"
#include "MacTargetSettings.h"
#include "XcodeProjectSettings.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"
#include "IMacTargetPlatformSettingsModule.h"

#if WITH_ENGINE
#include "CookedEditorTargetPlatformSettings.h"
#endif

#define LOCTEXT_NAMESPACE "FMacTargetPlatformSettingsModule"

/**
 * Module for Mac as a target platform settings
 */
class FMacTargetPlatformSettingsModule
	: public IMacTargetPlatformSettingsModule
{
public:
	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms) override
	{
		// Game TP
		ITargetPlatformSettings* GameTP = new TGenericMacTargetPlatformSettings<false, false, false>();
		TargetPlatforms.Add(GameTP);
		PlatformNameToPlatformSettings.Add(FMacPlatformProperties<false, false, false>::PlatformName(), GameTP);
		// Editor TP
		ITargetPlatformSettings* EditorTP = new TGenericMacTargetPlatformSettings<true, false, false>();
		TargetPlatforms.Add(EditorTP);
		PlatformNameToPlatformSettings.Add(FMacPlatformProperties<true, false, false>::PlatformName(), EditorTP);
		// Server TP
		ITargetPlatformSettings* ServerTP = new TGenericMacTargetPlatformSettings<false, true, false>();
		TargetPlatforms.Add(ServerTP);
		PlatformNameToPlatformSettings.Add(FMacPlatformProperties<false, true, false>::PlatformName(), ServerTP);
		// Client TP
		ITargetPlatformSettings* ClientTP = new TGenericMacTargetPlatformSettings<false, false, true>();
		TargetPlatforms.Add(ClientTP);
		PlatformNameToPlatformSettings.Add(FMacPlatformProperties<false, false, true>::PlatformName(), ClientTP);
        
#if WITH_ENGINE && COOKEDEDITOR_WITH_MACTARGETPLATFORM
        // currently this TP requires the engine for allowing GameDelegates usage
        bool bSupportCookedEditor;
        if (GConfig->GetBool(TEXT("CookedEditorSettings"), TEXT("bSupportCookedEditor"), bSupportCookedEditor, GGameIni) && bSupportCookedEditor)
        {
			ITargetPlatformSettings* CookedEditorTP = new TCookedEditorTargetPlatformSettings<FMacEditorTargetPlatformSettingsParent>();
			ITargetPlatformSettings* CookedCookerTP = new TCookedCookerTargetPlatformSettings<FMacEditorTargetPlatformSettingsParent>();
			PlatformSettingsCookedEditor = CookedEditorTP;
			PlatformSettingsCookedCooker = CookedCookerTP;
			TargetPlatforms.Add(CookedEditorTP);
			TargetPlatforms.Add(CookedCookerTP);
        }
#endif
	}

	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) override
	{
		OutMap = PlatformNameToPlatformSettings;
	}

	virtual ITargetPlatformSettings* GetCookedEditorPlatformSettings() override
	{
		return PlatformSettingsCookedEditor;
	}

	virtual ITargetPlatformSettings* GetCookedCookerPlatformSettings() override
	{
		return PlatformSettingsCookedCooker;
	}

public:

	// Begin IModuleInterface interface

	virtual void StartupModule() override
	{
#if WITH_ENGINE
		TargetSettings = NewObject<UMacTargetSettings>(GetTransientPackage(), "MacTargetSettings", RF_Standalone);
		
		// We need to manually load the config properties here, as this module is loaded before the UObject system is setup to do this
        GConfig->GetArray(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("TargetedRHIs"), TargetSettings->TargetedRHIs, GEngineIni);
       
        if (!GConfig->GetInt(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("MetalLanguageVersion"), TargetSettings->MetalLanguageVersion, GEngineIni))
        {
            TargetSettings->MetalLanguageVersion = 0;
        }
        
		if (!GConfig->GetBool(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("UseFastIntrinsics"), TargetSettings->UseFastIntrinsics, GEngineIni))
		{
			TargetSettings->UseFastIntrinsics = false;
		}
		
		if (!GConfig->GetBool(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("EnableMathOptimisations"), TargetSettings->EnableMathOptimisations, GEngineIni))
		{
			TargetSettings->EnableMathOptimisations = true;
		}
		
		TargetSettings->AddToRoot();
        
        ProjectSettings = NewObject<UXcodeProjectSettings>(GetTransientPackage(), "XcodeProjectSettings", RF_Standalone);
        ProjectSettings->AddToRoot();

		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "Mac",
				LOCTEXT("MacTargetSettingsName", "Mac"),
				LOCTEXT("MacTargetSettingsDescription", "Settings and resources for Mac platform"),
				TargetSettings
			);
            SettingsModule->RegisterSettings("Project", "Platforms", "Xcode",
                LOCTEXT("XcodeProjectSettingsName", "Xcode Projects"),
                LOCTEXT("XcodeProjectSettingsDescription", "Settings for Xcode projects"),
                ProjectSettings
            );
		}
#endif
	}

	virtual void ShutdownModule() override
	{
#if WITH_ENGINE
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "Mac");
            SettingsModule->UnregisterSettings("Project", "Platforms", "Xcode");
		}

		if (!GExitPurge)
		{
			// If we're in exit purge, this object has already been destroyed
			TargetSettings->RemoveFromRoot();
            ProjectSettings->RemoveFromRoot();
		}
		else
		{
			TargetSettings = NULL;
            ProjectSettings = NULL;
		}
#endif
	}

	// End IModuleInterface interface


private:
	TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;
	ITargetPlatformSettings* PlatformSettingsCookedEditor;
	ITargetPlatformSettings* PlatformSettingsCookedCooker;

	// Holds the target settings.
	UMacTargetSettings* TargetSettings;
    
    UXcodeProjectSettings* ProjectSettings;
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FMacTargetPlatformSettingsModule, MacTargetPlatformSettings);
