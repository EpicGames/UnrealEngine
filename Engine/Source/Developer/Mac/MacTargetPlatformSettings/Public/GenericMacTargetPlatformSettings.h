// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenericMacTargetPlatform.h: Declares the TGenericMacTargetPlatformSettings class template.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Common/TargetPlatformSettingsBase.h"
#include "Mac/MacPlatformProperties.h"

#if WITH_ENGINE
#include "RHI.h"
#include "TextureResource.h"
#include "StaticMeshResources.h"
#endif // WITH_ENGINE

#define LOCTEXT_NAMESPACE "TGenericMacTargetPlatformSettings"

/**
 * Template for Mac target platforms settings
 */
template<bool HAS_EDITOR_DATA, bool IS_DEDICATED_SERVER, bool IS_CLIENT_ONLY>
class TGenericMacTargetPlatformSettings
	: public TTargetPlatformSettingsBase<FMacPlatformProperties<HAS_EDITOR_DATA, IS_DEDICATED_SERVER, IS_CLIENT_ONLY> >
{
public:

	typedef FMacPlatformProperties<HAS_EDITOR_DATA, IS_DEDICATED_SERVER, IS_CLIENT_ONLY> TProperties;
	typedef TTargetPlatformSettingsBase<TProperties> TSuper;

	/**
	 * Default constructor.
	 */
	TGenericMacTargetPlatformSettings( )
	{
		#if WITH_ENGINE
			TextureLODSettings = nullptr;
			StaticMeshLODSettings.Initialize(this);
		#endif
	}

public:

	//~ Begin ITargetPlatform Interface

	virtual bool SupportsFeature( ETargetPlatformFeatures Feature ) const override
	{
		// we currently do not have a build target for MacServer
		if (Feature == ETargetPlatformFeatures::Packaging)
		{
			return (HAS_EDITOR_DATA || !IS_DEDICATED_SERVER);
		}

		if (Feature == ETargetPlatformFeatures::CanCookPackages)
		{
			return HAS_EDITOR_DATA;
		}
		
		if(Feature == ETargetPlatformFeatures::MobileRendering)
		{
			static bool bEnableMobileRenderingFeatureOnMac = false;

#if WITH_ENGINE
			static bool bHasCachedValue = false;
			if (!bHasCachedValue)
			{
				TArray<FName> TargetedShaderFormats;
				GetAllTargetedShaderFormats(TargetedShaderFormats);

				for (const FName& Format : TargetedShaderFormats)
				{
					if (IsMobilePlatform(ShaderFormatToLegacyShaderPlatform(Format)))
					{
						bEnableMobileRenderingFeatureOnMac = true;
						break;
					}
				}
				
				bHasCachedValue = true;
			}
#endif
			
			return bEnableMobileRenderingFeatureOnMac;
		}

		return TSuper::SupportsFeature(Feature);
	}

	virtual void GetAllPossibleShaderFormats( TArray<FName>& OutFormats ) const override
	{
		// no shaders needed for dedicated server target
		if (!IS_DEDICATED_SERVER)
		{
			static FName NAME_SF_METAL_SM5(TEXT("SF_METAL_SM5"));
			OutFormats.AddUnique(NAME_SF_METAL_SM5);
            static FName NAME_SF_METAL_SM6(TEXT("SF_METAL_SM6"));
            OutFormats.AddUnique(NAME_SF_METAL_SM6);
			static FName NAME_SF_METAL_ES3_1(TEXT("SF_METAL_ES3_1"));
			OutFormats.AddUnique(NAME_SF_METAL_ES3_1);
		}
	}

	virtual void GetAllTargetedShaderFormats(TArray<FName>& OutFormats) const override
	{
		// Get the Target RHIs for this platform, we do not always want all those that are supported.
		TArray<FString>TargetedShaderFormats;
		GConfig->GetArray(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("TargetedRHIs"), TargetedShaderFormats, GEngineIni);

		// Gather the list of Target RHIs and filter out any that may be invalid.
		TArray<FName> PossibleShaderFormats;
		GetAllPossibleShaderFormats(PossibleShaderFormats);

		for (int32 ShaderFormatIdx = TargetedShaderFormats.Num() - 1; ShaderFormatIdx >= 0; ShaderFormatIdx--)
		{
			FString ShaderFormat = TargetedShaderFormats[ShaderFormatIdx];
			if (PossibleShaderFormats.Contains(FName(*ShaderFormat)) == false)
			{
				TargetedShaderFormats.RemoveAt(ShaderFormatIdx);
			}
		}

		for(const FString& ShaderFormat : TargetedShaderFormats)
		{
			OutFormats.AddUnique(FName(*ShaderFormat));
		}
	}

#if WITH_ENGINE
	virtual const class FStaticMeshLODSettings& GetStaticMeshLODSettings( ) const override
	{
		return StaticMeshLODSettings;
	}

	virtual const UTextureLODSettings& GetTextureLODSettings() const override
	{
		return *TextureLODSettings;
	}

	virtual void RegisterTextureLODSettings(const UTextureLODSettings* InTextureLODSettings) override
	{
		TextureLODSettings = InTextureLODSettings;
	}

#endif //WITH_ENGINE

	virtual bool UsesRayTracing() const override
	{
		bool bEnableRayTracing = false;
		GConfig->GetBool(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("bEnableRayTracing"), bEnableRayTracing, GEngineIni);
	 
		return bEnableRayTracing;
	}
	
	virtual ERayTracingRuntimeMode GetRayTracingMode() const override
	{
		if (!TTargetPlatformSettingsBase<TProperties>::UsesRayTracing())
		{
			return ERayTracingRuntimeMode::Disabled;
		}

		return TTargetPlatformSettingsBase<TProperties>::ParseRuntimeRayTracingMode(TEXT("/Script/MacTargetPlatform.MacTargetSettings"));
	}

	//~ End ITargetPlatform Interface


	virtual bool IsRunningPlatform() const override
	{
		// Must be Mac platform as editor for this to be considered a running platform
		return PLATFORM_MAC && !UE_SERVER && !UE_GAME && WITH_EDITOR && HAS_EDITOR_DATA;
	}
private:

#if WITH_ENGINE
	// Holds the texture LOD settings.
	const UTextureLODSettings* TextureLODSettings;

	// Holds the static mesh LOD settings.
	FStaticMeshLODSettings StaticMeshLODSettings;

#endif // WITH_ENGINE

};

#undef LOCTEXT_NAMESPACE
